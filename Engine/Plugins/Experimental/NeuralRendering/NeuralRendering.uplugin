{
	"FileVersion": 3,
	"Version": 0.1,
	"VersionName": "0.1",
	"FriendlyName": "Neural Rendering",
	"Description": "Enable neural rendering features including: neural post processing",
	"Category": "Experimental",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"IsExperimentalVersion": true,
	"Installed": false,
	"SupportedTargetPlatforms": [
		"Win64",
		"Linux",
		"Mac"
	],
	"Modules": [
		{
			"Name": "NeuralPostProcessing",
			"Type": "RuntimeAndProgram",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [ "Win64", "Linux", "Mac" ]
		}
	],
	"Plugins": [
		{
			"Name": "NNERuntimeORT",
			"Enabled": true
		}
	]
}