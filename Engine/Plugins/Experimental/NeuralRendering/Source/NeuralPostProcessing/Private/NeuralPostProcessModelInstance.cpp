// Copyright Epic Games, Inc. All Rights Reserved.

#include "NeuralPostProcessModelInstance.h"
#include "NNERuntimeRDG.h"
#include "RenderGraphUtils.h"
#include "Engine/NeuralProfile.h"

TSharedPtr<UE::NNE::IModelInstanceRDG> CreateNNEModelInstance(UNNEModelData* NNEModelData, FString RuntimeName)
{
	check(NNEModelData);

	TWeakInterfacePtr<INNERuntime> Runtime = UE::NNE::GetRuntime<INNERuntime>(RuntimeName);
	TWeakInterfacePtr<INNERuntimeRDG> RuntimeRDG = UE::NNE::GetRuntime<INNERuntimeRDG>(RuntimeName);
	if (!Runtime.IsValid())
	{
#if WITH_EDITOR
		UE_LOGF(LogNeuralPostProcessing, Error, "Can't get %ls runtime.", *RuntimeName);
#endif
		return nullptr;
	}

	if (!RuntimeRDG.IsValid())
	{
#if WITH_EDITOR
		UE_LOGF(LogNeuralPostProcessing, Log, "No RDG runtime '%ls' found", *RuntimeName);
#endif
		return nullptr;
	}
	
	if (RuntimeRDG->CanCreateModelRDG(NNEModelData) != INNERuntimeRDG::ECanCreateModelRDGStatus::Ok)
	{
#if WITH_EDITOR
		UE_LOGF(LogNeuralPostProcessing, Error, "Cannot create model RDG for Model = %ls, Runtime = %ls", *NNEModelData->GetName(), *RuntimeName);
#endif
		return nullptr;
	}

	TSharedPtr<UE::NNE::IModelRDG> ModelRDG = RuntimeRDG->CreateModelRDG(NNEModelData);
	if (!ModelRDG.IsValid())
	{
#if WITH_EDITOR
		UE_LOGF(LogNeuralPostProcessing, Error, "CreateModelRDG failed for Model = %ls, Runtime = %ls", *NNEModelData->GetName(), *RuntimeName);
#endif
		return nullptr;
	}

	return ModelRDG->CreateModelInstanceRDG();
}

UNeuralPostProcessModelInstance::UNeuralPostProcessModelInstance(FObjectInitializer const& ObjectInitializer)
	:Super(ObjectInitializer)
{
	RDGInputBuffer = nullptr;
	RDGOutputBuffer = nullptr;
	RDGTiledInputBuffer = nullptr;
	RDGTiledOutputBuffer = nullptr;
	DispatchSize = 1;
	ModelTileSize = ENeuralModelTileType::OneByOne;
	DimensionOverride = FIntVector4(-1);
	TileOverlap = FIntPoint(0);
}

void UNeuralPostProcessModelInstance::Update(UNNEModelData* NNEModelData, FString RuntimeName)
{
	this->CreateDefaultNNEModel(NNEModelData, RuntimeName);
}

void UNeuralPostProcessModelInstance::Execute(FRDGBuilder& GraphBuilder)
{
	TArray<UE::NNE::FTensorBindingRDG> InputBindings;
	TArray<UE::NNE::FTensorBindingRDG> OutputBindings;
	UE::NNE::FTensorBindingRDG& Input = InputBindings.Emplace_GetRef();
	UE::NNE::FTensorBindingRDG& Output = OutputBindings.Emplace_GetRef();

	UE::NNE::FTensorShape InputShape = GetResolvedInputTensorShape();
	int32 InputBatchSize = InputShape.GetData()[0];

	auto RunSingleDispatch = [&](bool bAddFence = true) {
		Input.Buffer = RDGInputBuffer;
		Output.Buffer = RDGOutputBuffer;
		UE::NNE::IModelInstanceRDG::EEnqueueRDGStatus Status = ModelInstanceRDG->EnqueueRDG(GraphBuilder, InputBindings, OutputBindings);

		checkf(Status == UE::NNE::IModelInstanceRDG::EEnqueueRDGStatus::Ok, TEXT("EnqueueRDG failed: %d"), static_cast<int>(Status));

		if (bAddFence)
		{
			AddPass(GraphBuilder, RDG_EVENT_NAME("EnqueueRDGFence"), [this](FRHICommandListImmediate& RHICmdList)
				{
					// Need to wait until the output buffer, RDGOutputBuffer, is ready.
					RHICmdList.SubmitAndBlockUntilGPUIdle();
				});
		}
	};

	if (DispatchSize == 1)
	{
		RunSingleDispatch();
	}
	else if (DispatchSize > 1)
	{
		// Since we have more tiles than batch per dispatch we need to copy the data from tiled
		// to and from the buffer.
		int32 NumOfDispatch = DispatchSize;
		int32 InputBufferSize = RDGInputBuffer->Desc.GetSize();
		int32 OutputBufferSize = RDGOutputBuffer->Desc.GetSize();

		for (int i = 0; i < NumOfDispatch; ++i)
		{
			bool bAddFence = i == NumOfDispatch - 1;
			AddCopyBufferPass(GraphBuilder, RDGInputBuffer, 0, RDGTiledInputBuffer, InputBufferSize * i, InputBufferSize);
			RunSingleDispatch(bAddFence);
			AddCopyBufferPass(GraphBuilder, RDGTiledOutputBuffer, OutputBufferSize * i, RDGOutputBuffer, 0, OutputBufferSize);
		}
	}
}

UE::NNE::FTensorShape UNeuralPostProcessModelInstance::GetResolvedInputTensorShape()
{
	return ResolvedInputTensorShape;
}

UE::NNE::FTensorShape UNeuralPostProcessModelInstance::GetResolvedOutputTensorShape()
{
	return ResolvedOutputTensorShape;
}

FRDGBufferRef UNeuralPostProcessModelInstance::GetInputBuffer()
{
	return RDGInputBuffer;
}

FRDGBufferRef UNeuralPostProcessModelInstance::GetOutputBuffer()
{
	return RDGOutputBuffer;
}

FRDGBufferRef UNeuralPostProcessModelInstance::GetTiledInputBuffer()
{
	return RDGTiledInputBuffer;
}

FRDGBufferRef UNeuralPostProcessModelInstance::GetTiledOutputBuffer()
{
	return RDGTiledOutputBuffer;
}

bool UNeuralPostProcessModelInstance::ModifyInputShape(int Dim, int Size)
{
	// setup inputs
	TConstArrayView<UE::NNE::FTensorDesc> InputTensorDescs = ModelInstanceRDG->GetInputTensorDescs();

	UE::NNE::FSymbolicTensorShape InputShapeTemplate = InputTensorDescs[0].GetShape();
	checkf(InputShapeTemplate.Rank() == 4, TEXT("Post Processing requires models with input shape batch x channel x height x width!"));

	// We can only modify dimensions supporting dynamic Size
	UE::NNE::FTensorShape CurrentResolvedInputTensorShape = GetResolvedInputTensorShape();

	uint32 Batch = (Dim == 0) && (InputShapeTemplate.GetData()[0] < 0) ? Size : CurrentResolvedInputTensorShape.GetData()[0];
	uint32 Channel = (Dim == 1) && (InputShapeTemplate.GetData()[1] < 0) ? Size : CurrentResolvedInputTensorShape.GetData()[1];
	uint32 Height = (Dim == 2) && (InputShapeTemplate.GetData()[2] < 0) ? Size : CurrentResolvedInputTensorShape.GetData()[2];
	uint32 Width = (Dim == 3) && (InputShapeTemplate.GetData()[3] < 0) ? Size : CurrentResolvedInputTensorShape.GetData()[3];
	
	TArray<uint32> NewResolvedInputShape = {Batch,Channel,Height,Width};

	if (NewResolvedInputShape[Dim] != Size)
	{
#if WITH_EDITOR
		UE_LOGF(LogNeuralPostProcessing, Error, "Cannot set dimension %d to %d. It is not dynamic. revert back to %d",Dim, Size, NewResolvedInputShape[Dim]);
#endif
		return false;
	}

	ResolvedInputTensorShape = UE::NNE::FTensorShape::Make(NewResolvedInputShape);

	ModelInstanceRDG->SetInputTensorShapes({ ResolvedInputTensorShape });

	// Update the output shape as well due to the change of the input tensor shape

	TConstArrayView<UE::NNE::FTensorShape> ResolvedOutputTensorShapes = ModelInstanceRDG->GetOutputTensorShapes();
	if (ResolvedOutputTensorShapes.Num() < 1)
	{
#if WITH_EDITOR
		UE_LOGF(LogNeuralPostProcessing, Error, "Cannot set dimension %d to %d due to NNE error. Revert back to %d", Dim, Size, CurrentResolvedInputTensorShape.GetData()[Dim]);
#endif
		ModelInstanceRDG->SetInputTensorShapes({ CurrentResolvedInputTensorShape });
		return false;
	}

	ResolvedOutputTensorShape = ResolvedOutputTensorShapes[0];

	return true;
}

void UNeuralPostProcessModelInstance::CreateRDGBuffers(class FRDGBuilder& GraphBuilder)
{
	// Crate input buffers
	UE::NNE::FTensorShape InputShape = GetResolvedInputTensorShape();
	FIntPoint NeuralNetworkInputSize = { (int32) InputShape.GetData()[3], (int32) InputShape.GetData()[2] };
	int32 InputBatch = InputShape.GetData()[0];
	int32 InputChannels = InputShape.GetData()[1];

	FRDGBufferDesc InputBufferDesc = FRDGBufferDesc::CreateBufferDesc(sizeof(float), NeuralNetworkInputSize.X * NeuralNetworkInputSize.Y * InputChannels * InputBatch);
	
	RDGInputBuffer = GraphBuilder.CreateBuffer(InputBufferDesc, TEXT("NeuralPostProcessing.InputBuffer"));

	// Create output buffers
	checkf(ModelInstanceRDG->GetOutputTensorShapes().Num() >= 1, TEXT("Post Processing requires models with a single output tensor!"));
	
	ResolvedOutputTensorShape = ModelInstanceRDG->GetOutputTensorShapes()[0];
	checkf(ResolvedOutputTensorShape.Rank() == 4, TEXT("Neural Post Processing requires models dim = 4 [static/dynamic x C x height x width]!"));
	checkf(ResolvedOutputTensorShape.GetData()[0] != 0 , TEXT("Neural Post Processing requires models with output shape BatchSize != 0"));
	checkf(ResolvedOutputTensorShape.GetData()[1] >= 1, TEXT("Neural Post Processing requires models with output shape channel C >= 1!"));
	checkf(ResolvedOutputTensorShape.GetData()[2] > 0, TEXT("Neural Post Processing requires models with output height > 0!"));
	checkf(ResolvedOutputTensorShape.GetData()[3] > 0, TEXT("Neural Post Processing requires models with output width > 0!"));
	FIntPoint NeuralNetworkOutputSize = { (int32)ResolvedOutputTensorShape.GetData()[3], (int32)ResolvedOutputTensorShape.GetData()[2] };
	int32 OutputBatch = ResolvedOutputTensorShape.GetData()[0];
	int32 OutputChannels = ResolvedOutputTensorShape.GetData()[1];

	FRDGBufferDesc OutputBufferDesc = FRDGBufferDesc::CreateBufferDesc(sizeof(float), 
		NeuralNetworkOutputSize.X * NeuralNetworkOutputSize.Y * OutputChannels * OutputBatch);
	
	RDGOutputBuffer = GraphBuilder.CreateBuffer(OutputBufferDesc, *FString("SubsurfacePostProcessing.OutputBuffer"));

	if (DispatchSize <= 1)
	{
		RDGTiledInputBuffer = RDGInputBuffer;
		RDGTiledOutputBuffer = RDGOutputBuffer;
	}
	else
	{
		FRDGBufferDesc InputTiledBufferDesc = FRDGBufferDesc::CreateBufferDesc(sizeof(float), 
			NeuralNetworkInputSize.X * NeuralNetworkInputSize.Y * InputChannels * InputBatch * DispatchSize);
		RDGTiledInputBuffer = GraphBuilder.CreateBuffer(InputTiledBufferDesc, TEXT("NeuralPostProcessing.TiledInputBuffer"));

		FRDGBufferDesc OutputTiledBufferDesc = FRDGBufferDesc::CreateBufferDesc(sizeof(float), 
			NeuralNetworkOutputSize.X * NeuralNetworkOutputSize.Y * OutputChannels * OutputBatch * DispatchSize);
		RDGTiledOutputBuffer = GraphBuilder.CreateBuffer(OutputTiledBufferDesc, TEXT("NeuralPostProcessing.TiledOutputBuffer"));
	}

}

void UNeuralPostProcessModelInstance::CreateRDGBuffersIfNeeded(FRDGBuilder& GraphBuilder, bool bForceCreate)
{
	if (!RDGInputBuffer || !RDGOutputBuffer || bForceCreate)
	{
		this->CreateRDGBuffers(GraphBuilder);
	}
}

void UNeuralPostProcessModelInstance::CreateDefaultNNEModel(UNNEModelData* NNEModelData, FString RuntimeName)
{
	ModelInstanceRDG = CreateNNEModelInstance(NNEModelData, RuntimeName);

	if (ModelInstanceRDG)
	{
		// setup inputs
		TConstArrayView<UE::NNE::FTensorDesc> InputTensorDescs = ModelInstanceRDG->GetInputTensorDescs();

		// dimension check
		checkf(InputTensorDescs.Num() >= 1, TEXT("Post Processing requires models with a single input tensor!"));
		
		UE::NNE::FSymbolicTensorShape InputShape = InputTensorDescs[0].GetShape();
		checkf(InputShape.Rank() == 4, TEXT("Post Processing requires models with input shape 1 x N(N>=1) x height x width!"));
		checkf(InputShape.GetData()[0] != 0, TEXT("Post Processing requires models with input shape ? x N(N>=1) x height x width!"));
		checkf(InputShape.GetData()[1] >= 1, TEXT("Post Processing requires models with input shape ? x N(N>=1) x height x width!"));

		// Set up the input tensor shape
		// All dynamic dimensions are set to 1 by default.
		ResolvedInputTensorShape = UE::NNE::FTensorShape::MakeFromSymbolic(InputShape);

		if (ModelInstanceRDG->SetInputTensorShapes({ ResolvedInputTensorShape }) != UE::NNE::IModelInstanceRDG::ESetInputTensorShapesStatus::Ok)
		{
			ModelInstanceRDG.Reset();
#if WITH_EDITOR
			UE_LOGF(LogNeuralPostProcessing, Warning, "SetInputTensorShapes Failed for NNE RDG Model = %ls, Runtime = %ls", *NNEModelData->GetName(), *RuntimeName);
#endif
		}
		
	}
	else
	{
#if WITH_EDITOR
		UE_LOGF(LogNeuralPostProcessing, Log, "Failed to create NNE RDG Model.");
#endif
	}
}
