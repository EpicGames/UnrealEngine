// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "NNE.h"
#include "NNETypes.h"
#include "NNERuntimeRDG.h"
#include "NNEModelData.h"

#include "NeuralPostProcessModelInstance.generated.h"

#if WITH_EDITOR
	DECLARE_LOG_CATEGORY_EXTERN(LogNeuralPostProcessing, Log, All);
#endif

//Util functions
TSharedPtr<UE::NNE::IModelInstanceRDG> CreateNNEModelInstance(UNNEModelData* NNEModelData, FString RuntimeName);

enum class ENeuralModelTileType : uint8;
enum class ETileOverlapResolveType : uint8;

UCLASS()
class UNeuralPostProcessModelInstance : public UObject
{
	GENERATED_UCLASS_BODY()
public:

	// Engine thread 
	// If the model data and the Runtime name is different, recreate
	// the model
	void Update(UNNEModelData* NNEModelData, FString RuntimeName);

	// Render thread functions
	void CreateRDGBuffers(class FRDGBuilder& GraphBuilder);

	// Create the input and output buffer on demand
	// useful to use multiple post processing material to read/write
	void CreateRDGBuffersIfNeeded(class FRDGBuilder& GraphBuilder, bool bForceCreate = false);

	void Execute(class FRDGBuilder& GraphBuilder);

	UE::NNE::FTensorShape GetResolvedInputTensorShape();

	UE::NNE::FTensorShape GetResolvedOutputTensorShape();

	//TODO: different type support
	FRDGBufferRef GetInputBuffer();
	FRDGBufferRef GetOutputBuffer();

	FRDGBufferRef GetTiledInputBuffer();
	FRDGBufferRef GetTiledOutputBuffer();

	void UpdateDispatchSize(int InDispatchSize){ DispatchSize = InDispatchSize;}
	int	 GetDispatchSize() const{ return DispatchSize;}
	void UpdateTileDimension(FIntPoint InTileDim) { TileDim = InTileDim; };
	FIntPoint GetTileDimension() const { return TileDim; };
	void UpdateModelTileType(ENeuralModelTileType InTileType) { ModelTileSize = InTileType; }
	ENeuralModelTileType GetModelTileType() const { return ModelTileSize; }
	void UpdateTileOverlap(FIntPoint InTileOverlap){ TileOverlap = InTileOverlap;}
	FIntPoint GetTileOverlap() const { return TileOverlap;}
	void UpdateTileOverlapResolveType(ETileOverlapResolveType InTileOverlapResolveType) { TileOverlapResolveType = InTileOverlapResolveType; }
	ETileOverlapResolveType GetTileOverlapResolveType() const { return TileOverlapResolveType; }

	bool IsValid() { return ModelInstanceRDG.IsValid();}

	bool ModifyInputShape(int Dim, int Size);

private:
	void CreateDefaultNNEModel(UNNEModelData* NNEModelData, FString RuntimeName);

	// Input Buffer for neural post process network pass
	FRDGBufferRef RDGInputBuffer;
	
	// Output Buffer for Neural post process network pass
	FRDGBufferRef RDGOutputBuffer;

	// Temporary buffers to emulate batchsize
	FRDGBufferRef RDGTiledInputBuffer;

	FRDGBufferRef RDGTiledOutputBuffer;

	ENeuralModelTileType ModelTileSize;

	int DispatchSize = 1;
	FIntPoint TileDim = FIntPoint(1,1);

	// The NNE RDG Model 
	TSharedPtr<UE::NNE::IModelInstanceRDG> ModelInstanceRDG;

	// Resolved Input Tensorshape
	UE::NNE::FTensorShape ResolvedInputTensorShape;

	// Output Tensorshape
	UE::NNE::FTensorShape ResolvedOutputTensorShape;

	FIntVector4 DimensionOverride;

	FIntPoint TileOverlap;
	ETileOverlapResolveType TileOverlapResolveType;
};

