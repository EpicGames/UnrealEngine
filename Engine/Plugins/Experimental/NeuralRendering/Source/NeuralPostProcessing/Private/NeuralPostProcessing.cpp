// Copyright Epic Games, Inc. All Rights Reserved.

#include "NeuralPostProcessing.h"
#include "Interfaces/IPluginManager.h"
#include "NNE.h"
#include "NNEModelData.h"
#include "NNERuntimeRDG.h"
#include "Engine/NeuralProfile.h"
#include "NeuralPostProcessModelInstance.h"
#include "NeuralPostProcessingCS.h"
#include "PostProcess/NeuralPostProcessInterface.h"
#include "PixelShaderUtils.h"
#include "RenderGraphEvent.h"

#define LOCTEXT_NAMESPACE "FNeuralPostProcessingModule"

#if WITH_EDITOR
DEFINE_LOG_CATEGORY(LogNeuralPostProcessing);
#endif

using namespace NeuralPostProcessng;

namespace
{
	TAutoConsoleVariable<int32> CVarNeuralPostProcessApply(
		TEXT("r.Neuralpostprocess.Apply"),
		1,
		TEXT(" 0: disabled\n")
		TEXT(" 1: enabled (default)"),
		ECVF_RenderThreadSafe | ECVF_Scalability);

	int32 GNeuralPostProcessTileOverlap = -1;
	FAutoConsoleVariableRef CVarNeuralPostProcessTileOverlap(
		TEXT("r.Neuralpostprocess.TileOverlap"),
		GNeuralPostProcessTileOverlap,
		TEXT(" <0: Use the overlap from the profile\n"),
		ECVF_RenderThreadSafe | ECVF_Scalability);

	TAutoConsoleVariable<int32> CVarNeuralPostProcessTileOverlapResolveType(
		TEXT("r.Neuralpostprocess.TileOverlap.ResolveType"),
		-1,
		TEXT(" 0: Ignore overlapped region when concatinating the resolved subframe.\n")
		TEXT(" 1: Feathering overlapping regions based on bilinear filtering. Works best when Input and Output dimension are the same\n")
		TEXT(" <0: Use the ResolveType from profile."),
		ECVF_RenderThreadSafe | ECVF_Scalability);

	TAutoConsoleVariable<int32> CVarNeuralPostProcessTileOverlapVisualize(
		TEXT("r.Neuralpostprocess.TileOverlap.Visualize"),
		0,
		TEXT(" Visualize the overlap region if not zero\n"),
		ECVF_RenderThreadSafe | ECVF_Scalability);

	TAutoConsoleVariable<float> CVarNeuralPostProcessTileOverlapVisualizeIntensity(
		TEXT("r.Neuralpostprocess.TileOverlap.Visualize.Intensity"),
		1.0f,
		TEXT(" Adjust the intensity of the overlap for visualization\n"),
		ECVF_RenderThreadSafe | ECVF_Scalability);

	DECLARE_GPU_STAT(NeuralPostProcessing);
}

static FIntPoint ClampOverlap(FIntPoint TileOverlap,FIntPoint BufferSizeXY)
{
	// Use TileOverlap from CVar if larger or equal than zero.
	int32 OverlapFromCVar = GNeuralPostProcessTileOverlap;
	bool ShouldOverrideOverlapWithCVar = OverlapFromCVar >= 0;
	if (ShouldOverrideOverlapWithCVar)
	{
		TileOverlap = FIntPoint(OverlapFromCVar);
	}

	// Max tile border is 1/4 of the dimension. 
	FIntPoint MaxTileOverlap = BufferSizeXY / 4;
	TileOverlap.X = FMath::Clamp(TileOverlap.X, 0, MaxTileOverlap.X);
	TileOverlap.Y = FMath::Clamp(TileOverlap.Y, 0, MaxTileOverlap.Y);

	return TileOverlap;
}

/**
*
===============================================================================================================================
                    FNeuralPostProcessModelInstanceManager: Hold the mapping between neural profile and neural network models.
										/|\   /|\
										 | USE |
										 |     |
	|-ENGINE_API FNeuralProfileManager---|	   |
	|(INeuralProfileManager)			 |	   |
	|									 |	   |----RENDERING_API FNeuralPostProcess----|
	|									 |	   |	(INeuralPostProcessInterface)		|
	|	Manage the neural profile		 |	   |										|
	|									 |	   |										|
	|------------------------------------|	   |	Interface accessed by RDGBuilder	|
											   |	in post process material.			|
											   |----------------------------------------|
											   |    AllocateBuffer()                    |
											   |    Apply(): apply neural networks      |
===============================================================================================================================
*/

class FNeuralPostProcessModelInstanceManager
{
public:
	static FNeuralPostProcessModelInstanceManager* Get()
	{
		static FNeuralPostProcessModelInstanceManager Manager;
		return &Manager;
	}

	UNeuralPostProcessModelInstance* GetModelInstance(int32 ProfileId)
	{
		if (NeuralPostProcessModelInstances.Contains(ProfileId))
		{
			return NeuralPostProcessModelInstances[ProfileId];
		}
		else
		{
			return nullptr;
		}
	}

	void SetModelInstance(int32 ProfileId, UNeuralPostProcessModelInstance* Instance)
	{
		NeuralPostProcessModelInstances.Emplace(ProfileId, Instance);
	}

	UNeuralPostProcessModelInstance* Remove(int32 ProfileId)
	{
		return NeuralPostProcessModelInstances.FindAndRemoveChecked(ProfileId);
	}

	~FNeuralPostProcessModelInstanceManager()
	{
		NeuralPostProcessModelInstances.Reset();
	}
public:
	TMap<int32, UNeuralPostProcessModelInstance*> NeuralPostProcessModelInstances;
};

class FNeuralProfileManager : public NeuralProfile::INeuralProfileManager
{
public:
	FNeuralProfileManager() {}

	virtual void UpdateModel(int32 AllocationId, UObject* NNEModelData, FString RuntimeName) override
	{
		check(IsInRenderingThread())

		UNNEModelData* RawNNEModelData = Cast<UNNEModelData>(NNEModelData);

		if (!IsValid(RawNNEModelData))
		{
#if WITH_EDITOR
			UE_LOGF(LogNeuralPostProcessing, Error, "NNEModelData is invalid at Slot %d.", AllocationId);
#endif
			FNeuralPostProcessModelInstanceManager::Get()->SetModelInstance(AllocationId, nullptr);
			return;
		}

		UNeuralPostProcessModelInstance* ModelInstance = nullptr;
		{
			FGCScopeGuard GCScopeGuard;
			ModelInstance = NewObject<UNeuralPostProcessModelInstance>();
		}
		ModelInstance->Update(RawNNEModelData, RuntimeName);

		FNeuralPostProcessModelInstanceManager::Get()->SetModelInstance(AllocationId, ModelInstance->IsValid()? ModelInstance : nullptr);
	}

	virtual void UpdateTileType(int32 AllocationId, ENeuralModelTileType ModelTileType) override
	{
		UNeuralPostProcessModelInstance* Instance = FNeuralPostProcessModelInstanceManager::Get()->GetModelInstance(AllocationId);
		if (Instance)
		{
			Instance->UpdateModelTileType(ModelTileType);
		}
	}

	virtual bool UpdateBatchSize(int32 AllocationId, int32 BatchSize) override
	{
		UNeuralPostProcessModelInstance* Instance = FNeuralPostProcessModelInstanceManager::Get()->GetModelInstance(AllocationId);
		if (Instance)
		{
			return Instance->ModifyInputShape(0, BatchSize);
		}

		return false;
	}

	virtual void UpdateTileOverlap(int32 AllocationId, FIntPoint TileOverlap) override
	{
		UNeuralPostProcessModelInstance* Instance = FNeuralPostProcessModelInstanceManager::Get()->GetModelInstance(AllocationId);
		if (Instance)
		{
			UE::NNE::FTensorShape InputShape = Instance->GetResolvedInputTensorShape();
			FIntPoint BufferSizeXY = FIntPoint((int32)InputShape.GetData()[3], (int32)InputShape.GetData()[2]);
			Instance->UpdateTileOverlap(ClampOverlap(TileOverlap, BufferSizeXY));
		}
	}

	virtual void UpdateTileOverlapResolveType(int32 AllocationId, ETileOverlapResolveType TileOverlapResolveType) override
	{
		UNeuralPostProcessModelInstance* Instance = FNeuralPostProcessModelInstanceManager::Get()->GetModelInstance(AllocationId);
		if (Instance)
		{
			Instance->UpdateTileOverlapResolveType(TileOverlapResolveType);
		}
	}

	virtual void RemoveModel(int32 AllocationId) override
	{
		FNeuralPostProcessModelInstanceManager::Get()->Remove(AllocationId);
	}

	virtual FIntVector4 GetInputDimension(UObject* NNEModelData, FString RuntimeName) override
	{
		UE::NNE::FTensorShape TensorShape;
		FIntVector4 InputDimension = FIntVector4(-1, -1, -1, -1);

		if (UNNEModelData* ModelData = Cast<UNNEModelData>(NNEModelData))
		{
			// Need to create the ModelInstance in order to get the dimension in case the Model is not created
			// in the rendering thread.
			TSharedPtr<UE::NNE::IModelInstanceRDG> ModelInstance = CreateNNEModelInstance(ModelData, RuntimeName);

			if (ModelInstance)
			{
				TConstArrayView<UE::NNE::FTensorDesc> InputTensorDescs = ModelInstance->GetInputTensorDescs();
				UE::NNE::FSymbolicTensorShape InputTensorShape = InputTensorDescs[0].GetShape();

				// Support only output dimension of rank 4
				checkf(InputTensorShape.Rank() == 4, TEXT("Neural Post Processing requires models with input shape N x channel x height x width!"));

				InputDimension = FIntVector4(
					InputTensorShape.GetData()[0],
					InputTensorShape.GetData()[1],
					InputTensorShape.GetData()[2],
					InputTensorShape.GetData()[3]);
			}
			ModelInstance.Reset();
		}

		return InputDimension;
	}

	virtual FIntVector4 GetOutputDimension(UObject* NNEModelData, FString RuntimeName) override
	{
		UE::NNE::FTensorShape TensorShape;
		FIntVector4 OutputDimension = FIntVector4(-1,-1,-1,-1);

		if (UNNEModelData* ModelData = Cast<UNNEModelData>(NNEModelData))
		{
			TSharedPtr<UE::NNE::IModelInstanceRDG> ModelInstance = CreateNNEModelInstance(ModelData, RuntimeName);
			
			if (ModelInstance)
			{
				TConstArrayView<UE::NNE::FTensorDesc> OutputTensorDescs = ModelInstance->GetOutputTensorDescs();
				UE::NNE::FSymbolicTensorShape OutputTensorShape = OutputTensorDescs[0].GetShape();

				checkf(OutputTensorShape.Rank() == 4, TEXT("Neural Post Processing requires models with output shape N x channel x height x width!"));

				OutputDimension = FIntVector4(
					OutputTensorShape.GetData()[0],
					OutputTensorShape.GetData()[1],
					OutputTensorShape.GetData()[2],
					OutputTensorShape.GetData()[3]);
			}

			ModelInstance.Reset();
		}
		return OutputDimension;
	}

	~FNeuralProfileManager() {}
};


int32 GetTotalModelTileCount(ENeuralModelTileType ModelTileSize)
{
	int TotalTileCount = 1;

	//@TODO: uncomment and implement the corresponding function.
	switch (ModelTileSize)
	{
	case ENeuralModelTileType::TwoByTwo:
		TotalTileCount = 2 * 2;
		break;
	case ENeuralModelTileType::FourByFour:
		TotalTileCount = 4 * 4;
		break;
	case ENeuralModelTileType::EightByEight:
		TotalTileCount = 8 * 8;
		break;
	case ENeuralModelTileType::Auto:
		TotalTileCount = -1;	// determined at runtime by the viewport size and W,H dimension of the network 
		break;
	case ENeuralModelTileType::OneByOne:
	default:
		break;
	}

	return TotalTileCount;
}

static void AllocateInputBuffer_RenderingThread(
	FRDGBuilder& GraphBuilder,
	const FScreenPassTextureViewport& Viewport,
	int32 ProfileId,
	FRDGBufferRef& InputNeuralBuffer,
	FVector4f& InputBufferDimension)
{
	check(IsInRenderingThread())
		check(ProfileId >= 0);

	UNeuralPostProcessModelInstance* Model = FNeuralPostProcessModelInstanceManager::Get()->GetModelInstance(ProfileId);

	if (!IsValid(Model))
	{
		InputNeuralBuffer = nullptr;
		return;
	}

	UE::NNE::FTensorShape InputShape = Model->GetResolvedInputTensorShape();
	InputBufferDimension = FVector4f(InputShape.GetData()[0], InputShape.GetData()[1], InputShape.GetData()[2], InputShape.GetData()[3]);
	ENeuralModelTileType ModelTileType = Model->GetModelTileType();
	FScreenPassTextureViewportParameters ViewportParameters = GetScreenPassTextureViewportParameters(Viewport);
	int32 BatchDim = InputBufferDimension.X;

	// Calculate the number of tiles
	int NumDispatches = 1;
	FIntPoint TileDimension = FIntPoint(1, 1);
	{	
		if (ModelTileType == ENeuralModelTileType::Auto)
		{
			// Allocate the number of tiles based on the size of the viewport, the size of the buffer and tile configurations.
			FIntPoint InputBufferSizeXY = FIntPoint(InputBufferDimension.W, InputBufferDimension.Z);
			FIntPoint TileOvelapBorder = ClampOverlap(Model->GetTileOverlap(), InputBufferSizeXY);
			FIntPoint EffectiveBufferWH = InputBufferSizeXY - TileOvelapBorder * 2;
			FVector2f TileSizeWH = FMath::DivideAndRoundUp(ViewportParameters.ViewportSize.IntPoint(), EffectiveBufferWH);
			TileDimension = FIntPoint(TileSizeWH.X,TileSizeWH.Y);
			NumDispatches = FMath::DivideAndRoundUp(TileDimension.X * TileDimension.Y, BatchDim);
		}
		else
		{
			// Use fixed number of tiles
			int32 TileCount = GetTotalModelTileCount(ModelTileType);
			int32 SideLength = (int32)FMath::Sqrt((float)TileCount);
			TileDimension = FIntPoint(SideLength, SideLength);
			NumDispatches = FMath::DivideAndRoundUp(TileCount, BatchDim);
		}

		Model->UpdateDispatchSize(NumDispatches);
		Model->UpdateTileDimension(TileDimension);
	}
	
	// Create the buffer
	Model->CreateRDGBuffersIfNeeded(GraphBuilder, true);


	// Output the buffer and dimension for use.
	InputNeuralBuffer = Model->GetTiledInputBuffer();
	InputBufferDimension.X *= NumDispatches;

	// Clear the input buffer at the border if we use ENeuralModelTileType::Auto
	// and the botom and right tiles partially covers the outside.
	if (ModelTileType == ENeuralModelTileType::Auto)
	{
		//@TODO: only clear part of the regions at borders.
		AddClearUAVPass(GraphBuilder,GraphBuilder.CreateUAV(InputNeuralBuffer,PF_R32_FLOAT),0);
	}
}

FCopyBetweenTextureAndOverlappedTileBufferCS::EOverlapResolveType GetOverlapResolveType(FIntPoint TileOverlap, ETileOverlapResolveType TileOverlapResolveType)
{
	typedef FCopyBetweenTextureAndOverlappedTileBufferCS SHADER;

	SHADER::EOverlapResolveType OverlapResolveType = SHADER::EOverlapResolveType::Ignore;

	// Set TileOverlap Resolve type by profile config
	switch (TileOverlapResolveType)
	{
	case ETileOverlapResolveType::Feathering:
		OverlapResolveType = SHADER::EOverlapResolveType::Feathering;
		break;
	case ETileOverlapResolveType::Ignore:
	default:
		break;
	}

	// If both overlaps are zero, switch to Ignore for performance
	if (OverlapResolveType == SHADER::EOverlapResolveType::Feathering && (TileOverlap.X == 0 && TileOverlap.Y == 0))
	{
		OverlapResolveType = SHADER::EOverlapResolveType::Ignore;
	}

	// Override by console variables
	int32 OverlapResolveTypeOverride = CVarNeuralPostProcessTileOverlapResolveType.GetValueOnRenderThread();
	if (OverlapResolveTypeOverride == 0)
	{
		OverlapResolveType = SHADER::EOverlapResolveType::Ignore;
	}
	else if (OverlapResolveTypeOverride > 0)
	{
		OverlapResolveType = SHADER::EOverlapResolveType::Feathering;
	}

	return OverlapResolveType;
}

FIntPoint ScaleValueFromSourceToTargetSize(FIntPoint Value, FIntPoint SourceSize, FIntPoint TargetSize)
{
	FVector2f ScaledValue = FVector2f(Value.X * TargetSize.X * 1.0f / SourceSize.X, Value.Y * TargetSize.Y * 1.0f / SourceSize.Y);
	return FIntPoint(FMath::CeilToInt(ScaledValue.X), FMath::CeilToInt(ScaledValue.Y));
}

FIntPoint GetScaledTileOverlap(FIntPoint TileOverlap, FIntPoint SourceSize, FIntPoint TargetSize)
{
	// Scaled overlap is designed to return integer.
	return ScaleValueFromSourceToTargetSize(TileOverlap, SourceSize, TargetSize);
}

static void	ApplyNeuralNetworks_RenderingThread(
	FRDGBuilder& GraphBuilder,
	int32 ProfileId,
	FRDGTextureRef NeuralTexture,
	FIntRect ViewRect,
	FRDGBufferRef InputSourceType,
	FRDGBufferRef& OutputNeuralBuffer,
	FVector4f& BufferDimension)
{
	check(IsInRenderingThread())
		check(ProfileId >= 0);

	UNeuralPostProcessModelInstance* Model = FNeuralPostProcessModelInstanceManager::Get()->GetModelInstance(ProfileId);

	if (!IsValid(Model))
	{
		return;
	}

	RDG_EVENT_SCOPE_STAT(GraphBuilder, NeuralPostProcessing, "NeuralPostProcessing");

	FIntPoint TextureSize = NeuralTexture->Desc.Extent;
	FGlobalShaderMap* GlobalShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
	float Scale = 1.0;

	UE::NNE::FTensorShape InputShape = Model->GetResolvedInputTensorShape();

	FIntPoint NeuralNetworkInputSize = { (int32)InputShape.GetData()[3], (int32)InputShape.GetData()[2] };// Width, height
	FIntPoint TileOverlap = ClampOverlap(Model->GetTileOverlap(), NeuralNetworkInputSize);
	FIntPoint TileDimension = Model->GetTileDimension();
	
	const FScreenPassTextureViewport NeuralPostProcessViewport(NeuralTexture, ViewRect);
	const FScreenPassTextureViewportParameters InputViewportParameters = GetScreenPassTextureViewportParameters(NeuralPostProcessViewport);
	
	const ENeuralModelTileType ModelTileType = Model->GetModelTileType();
	const bool bRequireInputBufferScale = ModelTileType != ENeuralModelTileType::Auto;
	const ETileOverlapResolveType TileOverlapResolveType = Model->GetTileOverlapResolveType();

	struct FIntermediateTexture
	{
		FRDGTextureRef Texture;
		FIntPoint		Extent;
	};

	auto GetIntermediateTexture = [&](FIntPoint InNetworkSize, FIntPoint InOverLap, bool bRequireBufferScale, const TCHAR* Name)->FIntermediateTexture {
		FIntermediateTexture InputIntermediateTexture;

		if (bRequireBufferScale)
		{
			FIntPoint IntermediateTextureExtent = (InNetworkSize - InOverLap * 2) * TileDimension;
			FRDGTextureDesc IntermediateTextureDesc = NeuralTexture->Desc;
			IntermediateTextureDesc.Extent = IntermediateTextureExtent;

			// @TODO: conditionally create the intermediate buffer texture
			InputIntermediateTexture.Texture = GraphBuilder.CreateTexture(IntermediateTextureDesc, Name);
			InputIntermediateTexture.Extent = IntermediateTextureExtent;
		}
		else
		{
			InputIntermediateTexture.Texture = NeuralTexture;
			InputIntermediateTexture.Extent = InputViewportParameters.ViewportSize.IntPoint();
		}
		return InputIntermediateTexture;
	};

	// 1. Preprocess the input data by copying from Texture to buffer if required
	{
		// Build the indirect dispatch parameters
		FRDGBufferRef IndirectDispatchBuffer = GraphBuilder.CreateBuffer(
			FRDGBufferDesc::CreateIndirectDesc<FRHIDispatchIndirectParameters>(1), TEXT("NeuralPostProcessing.IndirectDispatchBuffer"));

		FIntermediateTexture IntermediateTexture = 
			GetIntermediateTexture(NeuralNetworkInputSize, TileOverlap, bRequireInputBufferScale, TEXT("NeuralPostProcessing.IntermediateBufferTexture"));

		// Downscale
		if (bRequireInputBufferScale)
		{
			// Build the indirect dispatch parameters with InputSourceType
			{
				typedef FNeuralPostProcessingBuildIndirectDispatchArgsCS ARGSETUPSHADER;
				ARGSETUPSHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<ARGSETUPSHADER::FParameters>();
				PassParameters->TargetDimension = IntermediateTexture.Extent;
				PassParameters->SourceType = GraphBuilder.CreateSRV(InputSourceType, EPixelFormat::PF_R32_UINT);
				PassParameters->RWIndirectDispatchArgsBuffer = GraphBuilder.CreateUAV(IndirectDispatchBuffer, EPixelFormat::PF_R32_UINT);
				TShaderMapRef<ARGSETUPSHADER> ComputeShader(GlobalShaderMap);
				FComputeShaderUtils::AddPass(GraphBuilder, FRDGEventName(TEXT("NeuralPostProcessing::BuildIndirectArgs(Dispatch)")), ComputeShader, PassParameters, FIntVector(1, 1, 1));
			}

			typedef FDownScaleTextureCS SHADER;
			SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
			{
				PassParameters->Source = GetNeuralPostProcessInput(NeuralTexture, InputViewportParameters);
				PassParameters->SourceTextureSampler = TStaticSamplerState<SF_Bilinear, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
				PassParameters->TargetWidth = IntermediateTexture.Extent.X;
				PassParameters->TargetHeight = IntermediateTexture.Extent.Y;
				PassParameters->TargetTexture = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(IntermediateTexture.Texture));
				PassParameters->IndirectDispatchArgsBuffer = IndirectDispatchBuffer;
			}
			TShaderMapRef<SHADER> ComputeShader(GlobalShaderMap);
			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("NeuralPostProcessing::DownScale (%dx%d -> %dx%d)",
					(int32)InputViewportParameters.ViewportSize.X,
					(int32)InputViewportParameters.ViewportSize.Y,
					IntermediateTexture.Extent.X,
					IntermediateTexture.Extent.Y),
				ERDGPassFlags::Compute, //| ERDGPassFlags::NeverCull
				ComputeShader,
				PassParameters,
				PassParameters->IndirectDispatchArgsBuffer, 0);
		}

		// Copy from Texture to Overlapped tile buffer
		{
			int32 NumOfChannel = (int32)InputShape.GetData()[1];
			// Build the indirect dispatch parameters with InputSourceType
			{
				typedef FNeuralPostProcessingBuildIndirectDispatchArgsCS ARGSETUPSHADER;
				ARGSETUPSHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<ARGSETUPSHADER::FParameters>();
				PassParameters->TargetDimension = NeuralNetworkInputSize * TileDimension;
				PassParameters->SourceType = GraphBuilder.CreateSRV(InputSourceType, EPixelFormat::PF_R32_UINT);
				PassParameters->RWIndirectDispatchArgsBuffer = GraphBuilder.CreateUAV(IndirectDispatchBuffer, EPixelFormat::PF_R32_UINT);
				TShaderMapRef<ARGSETUPSHADER> ComputeShader(GlobalShaderMap);
				FComputeShaderUtils::AddPass(GraphBuilder, FRDGEventName(TEXT("NeuralPostProcessing::BuildIndirectArgs(Dispatch)")), ComputeShader, PassParameters, FIntVector(1, 1, 1));
			}

			typedef FCopyBetweenTextureAndOverlappedTileBufferCS SHADER;
			SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
			{
				PassParameters->SourceWidth = IntermediateTexture.Extent.X;
				PassParameters->SourceHeight = IntermediateTexture.Extent.Y;
				PassParameters->RWSourceTexture = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(IntermediateTexture.Texture));
				PassParameters->SourceTextureSampler = TStaticSamplerState<SF_Point, AM_Mirror, AM_Mirror, AM_Mirror>::GetRHI();
				
				PassParameters->TargetOverlappedTileWidth = NeuralNetworkInputSize.X;
				PassParameters->TargetOverlappedTileHeight = NeuralNetworkInputSize.Y;
				PassParameters->ViewTileDimension = TileDimension;
				PassParameters->TileOverlap = FVector2f(TileOverlap.X,TileOverlap.Y);
				PassParameters->NumOfChannel = NumOfChannel;
				PassParameters->TargetBuffer = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(Model->GetTiledInputBuffer(), PF_R32_FLOAT));
				PassParameters->IndirectDispatchArgsBuffer = IndirectDispatchBuffer;
			}

			SHADER::FPermutationDomain ComputeShaderPermutationVector;
			ComputeShaderPermutationVector.Set<SHADER::FDimensionCopyDirection>(SHADER::EDirection::ToOverlappedTiles);
			ComputeShaderPermutationVector.Set<SHADER::FDimensionOverlapResolveType>(SHADER::EOverlapResolveType::Ignore);
			TShaderMapRef<SHADER> ComputeShader(GlobalShaderMap,ComputeShaderPermutationVector);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("NeuralPostProcessing::CopyToTiledBuffer (TileOverlap=%d,%d)",TileOverlap.X,TileOverlap.Y),
				ERDGPassFlags::Compute,
				ComputeShader,
				PassParameters,
				PassParameters->IndirectDispatchArgsBuffer, 0);
		}

	}

	// 2. Run the network
	const bool bNeuralPostProcessApply = CVarNeuralPostProcessApply.GetValueOnRenderThread() > 0;
	if (bNeuralPostProcessApply)
	{
		Model->Execute(GraphBuilder);
	}

	// 3. Pass the output from the neural network and fill the scene color texture
	// when batch and channel dimension matches between the input and output buffer.

	auto ShouldUpdateNeuralTexture = [&]()->bool {
		UE::NNE::FTensorShape OutputShape = Model->GetResolvedOutputTensorShape();
		UE::NNE::FTensorShape InputShape = Model->GetResolvedInputTensorShape();

		return OutputShape.GetData()[0] == InputShape.GetData()[0] &&
			OutputShape.GetData()[1] == InputShape.GetData()[1] &&
			Model->GetDispatchSize() >= 1;
	};

	if (ShouldUpdateNeuralTexture())
	{
		FRDGBufferRef OutputBuffer = bNeuralPostProcessApply ? Model->GetTiledOutputBuffer() : Model->GetTiledInputBuffer();
		FRDGBufferUAVRef OutputBufferUAV = GraphBuilder.CreateUAV(FRDGBufferUAVDesc(OutputBuffer, PF_R32_FLOAT));

		UE::NNE::FTensorShape OutputShape = bNeuralPostProcessApply ? Model->GetResolvedOutputTensorShape() : Model->GetResolvedInputTensorShape();
		FIntPoint NeuralNetworkOutputSize = { (int32)OutputShape.GetData()[3], (int32)OutputShape.GetData()[2] };

		FIntPoint ScaledTileOverlap = GetScaledTileOverlap(TileOverlap, NeuralNetworkInputSize, NeuralNetworkOutputSize);

		// Need to scale output buffer if the output network size is larger than that of the input of the network.
		const bool bRequireOutputBufferScale = ModelTileType != ENeuralModelTileType::Auto ||
			(NeuralNetworkOutputSize.X != NeuralNetworkInputSize.X || NeuralNetworkOutputSize.Y != NeuralNetworkInputSize.Y);

		FIntermediateTexture IntermediateTexture = 
			GetIntermediateTexture(NeuralNetworkOutputSize, ScaledTileOverlap, bRequireOutputBufferScale, TEXT("NeuralPostProcessing.IntermediateOutputBufferTexture"));
		
		// Copy from overlapped tile buffer to texture
		{
			int32 NumOfChannel = (int32)InputShape.GetData()[1];

			typedef FCopyBetweenTextureAndOverlappedTileBufferCS SHADER;
			SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
			{
				PassParameters->SourceWidth = IntermediateTexture.Extent.X;
				PassParameters->SourceHeight = IntermediateTexture.Extent.Y;
				PassParameters->RWSourceTexture = GraphBuilder.CreateUAV(FRDGTextureUAVDesc(IntermediateTexture.Texture));
				PassParameters->SourceTextureSampler = TStaticSamplerState<SF_Point, AM_Mirror, AM_Mirror, AM_Mirror>::GetRHI();

				PassParameters->TargetOverlappedTileWidth = NeuralNetworkOutputSize.X;
				PassParameters->TargetOverlappedTileHeight = NeuralNetworkOutputSize.Y;
				PassParameters->ViewTileDimension = TileDimension;
				PassParameters->TileOverlap = FVector2f(ScaledTileOverlap.X, ScaledTileOverlap.Y);
				PassParameters->NumOfChannel = NumOfChannel;
				PassParameters->bVisualizeOverlap = CVarNeuralPostProcessTileOverlapVisualize.GetValueOnRenderThread() != 0;
				PassParameters->OverlapVisualizeIntensity = FMath::Max(0.0001, CVarNeuralPostProcessTileOverlapVisualizeIntensity.GetValueOnRenderThread());
				PassParameters->TargetBuffer = OutputBufferUAV;
			}

			SHADER::FPermutationDomain ComputeShaderPermutationVector;
			ComputeShaderPermutationVector.Set<SHADER::FDimensionCopyDirection>(SHADER::EDirection::FromOverlappedTiles);
			ComputeShaderPermutationVector.Set<SHADER::FDimensionOverlapResolveType>(GetOverlapResolveType(TileOverlap, TileOverlapResolveType));
			TShaderMapRef<SHADER> ComputeShader(GlobalShaderMap, ComputeShaderPermutationVector);

			FComputeShaderUtils::AddPass(
				GraphBuilder,
				RDG_EVENT_NAME("NeuralPostProcessing::CopyFromTiledBuffer (TileOverlap=%d,%d)", TileOverlap.X,TileOverlap.Y),
				ERDGPassFlags::Compute,
				ComputeShader,
				PassParameters,
				FComputeShaderUtils::GetGroupCount(NeuralNetworkOutputSize * TileDimension, NEURAL_POST_PROCESSING_THREAD_GROUP_SIZE));
		}

		// Scale and write to the target texture
		if (bRequireOutputBufferScale)
		{
			FIntPoint SourceExtent = IntermediateTexture.Extent;
			
			// Update the effective source width and height if we use Auto.
			if (ModelTileType == ENeuralModelTileType::Auto)
			{
				int ModX = InputViewportParameters.ViewportSize.IntPoint().X % (NeuralNetworkInputSize.X - 2 * TileOverlap.X);
				int ModY = InputViewportParameters.ViewportSize.IntPoint().Y % (NeuralNetworkInputSize.Y - 2 * TileOverlap.Y);
				FIntPoint ScaledSize = ScaleValueFromSourceToTargetSize(FIntPoint(ModX, ModY), NeuralNetworkInputSize, NeuralNetworkOutputSize);
				SourceExtent = (NeuralNetworkOutputSize - ScaledTileOverlap * 2 )* (TileDimension - 1) + ScaledSize;
			}

			const bool bUpScale = SourceExtent[0] < ViewRect.Size()[0] && SourceExtent[1] < ViewRect.Size()[1];

			if (bUpScale)
			{
				typedef FUpscaleTexture SHADER;
				SHADER::FParameters* ProcessOutputParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
				ProcessOutputParameters->Source_Texture = IntermediateTexture.Texture;
				ProcessOutputParameters->ViewportSize = InputViewportParameters.ViewportSize.IntPoint();
				ProcessOutputParameters->SourceWidth = SourceExtent.X;
				ProcessOutputParameters->SourceHeight = SourceExtent.Y;
				ProcessOutputParameters->RenderTargets[0] = FRenderTargetBinding(NeuralTexture, ERenderTargetLoadAction::ELoad);

				TShaderMapRef<SHADER> WriteOutputShader(GlobalShaderMap);

				FPixelShaderUtils::AddFullscreenPass(
					GraphBuilder,
					GlobalShaderMap,
					RDG_EVENT_NAME("NeuralPostProcessing::Upscale (%dx%d -> %dx%d)",
						SourceExtent.X,
						SourceExtent.Y,
						(int32)InputViewportParameters.ViewportSize.X,
						(int32)InputViewportParameters.ViewportSize.Y),
					WriteOutputShader,
					ProcessOutputParameters,
					ViewRect);
			}
			else // Down Scale
			{
				FScreenPassTextureViewport IntermediateTextureViewport(IntermediateTexture.Texture, FIntRect(0, 0, SourceExtent.X, SourceExtent.Y));
				FScreenPassTextureViewportParameters IntermediateTextureViewportParameters = GetScreenPassTextureViewportParameters(IntermediateTextureViewport);

				typedef FDownScaleTexture SHADER;
				SHADER::FParameters* PassParameters = GraphBuilder.AllocParameters<SHADER::FParameters>();
				{
					PassParameters->Source = GetNeuralPostProcessInput(IntermediateTexture.Texture, IntermediateTextureViewportParameters);
					PassParameters->SourceTextureSampler = TStaticSamplerState<SF_Point, AM_Clamp, AM_Clamp, AM_Clamp>::GetRHI();
					PassParameters->TargetWidth = InputViewportParameters.ViewportSize.X;
					PassParameters->TargetHeight = InputViewportParameters.ViewportSize.Y;
					PassParameters->RenderTargets[0] = FRenderTargetBinding(NeuralTexture, ERenderTargetLoadAction::ELoad);
				}
				TShaderMapRef<SHADER> WriteOutputShader(GlobalShaderMap);

				FPixelShaderUtils::AddFullscreenPass(
					GraphBuilder,
					GlobalShaderMap,
					RDG_EVENT_NAME("NeuralPostProcessing::Downscale (%dx%d -> %dx%d)",
						SourceExtent.X,
						SourceExtent.Y,
						(int32)InputViewportParameters.ViewportSize.X,
						(int32)InputViewportParameters.ViewportSize.Y),
					WriteOutputShader,
					PassParameters,
					ViewRect);
			}
		}
	}

	// 4. Read back the buffer, so the user can directly decode in the post process material.
	OutputNeuralBuffer = bNeuralPostProcessApply ? Model->GetTiledOutputBuffer() : Model->GetTiledInputBuffer();
	UE::NNE::FTensorShape OutputShape = bNeuralPostProcessApply ? Model->GetResolvedOutputTensorShape() : Model->GetResolvedInputTensorShape();		
	BufferDimension = FVector4f(OutputShape.GetData()[0], OutputShape.GetData()[1], OutputShape.GetData()[2], OutputShape.GetData()[3]);
	BufferDimension.X *= Model->GetDispatchSize();
}

class FNeuralPostProcess : public INeuralPostProcessInterface
{
public:
	virtual void Apply(FRDGBuilder& GraphBuilder, int32 NeuralProfileId,
		FRDGTexture* NeuralTexture, FIntRect ViewRect, FRDGBufferRef InputSourceType,
		FRDGBufferRef& OutputNeuralBuffer, FVector4f& BufferDimension) override
	{
		ApplyNeuralNetworks_RenderingThread(
			GraphBuilder,
			NeuralProfileId,
			NeuralTexture,
			ViewRect,
			InputSourceType,
			OutputNeuralBuffer,
			BufferDimension);
	}

	virtual void AllocateBuffer(FRDGBuilder& GraphBuilder, const FScreenPassTextureViewport& Viewport,
		int32 NeuralProfileId, FRDGBufferRef& InputNeuralBuffer, FVector4f& InputBufferDimension) override
	{
		AllocateInputBuffer_RenderingThread(
			GraphBuilder, 
			Viewport, 
			NeuralProfileId, 
			InputNeuralBuffer, 
			InputBufferDimension);
	}
};

void FNeuralPostProcessingModule::StartupModule()
{
#if WITH_EDITOR
	UE_LOGF(LogNeuralPostProcessing, Log, "NeuralPostProcessing starting up");
#endif

	const TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(TEXT("NeuralRendering"));
	if (Plugin.IsValid())
	{
		FString ModuleDir = Plugin->GetBaseDir() + TEXT("/Source/NeuralPostProcessing");
		AddShaderSourceDirectoryMapping(TEXT("/NeuralRendering"), FPaths::Combine(ModuleDir, TEXT("Shaders")));
	}
	else
	{
#if WITH_EDITOR
		UE_LOGF(LogNeuralPostProcessing, Error, "Shaders directory not added. Failed to find NeuralPostProcessing plugin");
#endif
	}

	GNeuralProfileManager.Reset(new FNeuralProfileManager());
	GNeuralPostProcess.Reset(new FNeuralPostProcess());
}

void FNeuralPostProcessingModule::ShutdownModule()
{
#if WITH_EDITOR
	UE_LOGF(LogNeuralPostProcessing, Log, "NeuralPostProcessing function shutting down");
#endif

	GNeuralProfileManager.Reset(nullptr);
	GNeuralPostProcess.Reset(nullptr);
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FNeuralPostProcessingModule, NeuralPostProcessing)