// Copyright Epic Games, Inc. All Rights Reserved.


#include "NeuralPostProcessingCS.h"
#include "ShaderCompilerCore.h"

namespace NeuralPostProcessng
{
	FNueralPostProcessInput GetNeuralPostProcessInput(FRDGTextureRef Texture, const FScreenPassTextureViewportParameters& ViewportParameters)
	{
		FNueralPostProcessInput Input;
		Input.Texture = Texture;
		Input.Viewport = ViewportParameters;
		return Input;
	}

	void FNeuralPostProcessingBuildIndirectDispatchArgsCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), NEURAL_POST_PROCESSING_THREAD_GROUP_SIZE);
	}

	bool FNeuralPostProcessingBuildIndirectDispatchArgsCS::ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	void FDownScaleTextureCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), NEURAL_POST_PROCESSING_THREAD_GROUP_SIZE);
	}

	void FDownScaleTexture::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
	}

	void FUpscaleTexture::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), NEURAL_POST_PROCESSING_THREAD_GROUP_SIZE);
	}

	void FCopyBetweenTextureAndOverlappedTileBufferCS::ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(InParameters, OutEnvironment);
		OutEnvironment.SetDefine(TEXT("THREAD_GROUP_SIZE"), NEURAL_POST_PROCESSING_THREAD_GROUP_SIZE);
		OutEnvironment.CompilerFlags.Add(CFLAG_AllowTypedUAVLoads);
	}

	IMPLEMENT_GLOBAL_SHADER(FNeuralPostProcessingBuildIndirectDispatchArgsCS, "/NeuralRendering/NeuralPostProcessing.usf", "BuildIndirectDispatchArgsCS", SF_Compute);

	IMPLEMENT_GLOBAL_SHADER(FDownScaleTextureCS, "/NeuralRendering/NeuralPostProcessing.usf", "DownScaleTextureCS", SF_Compute);
	IMPLEMENT_GLOBAL_SHADER(FDownScaleTexture, "/NeuralRendering/NeuralPostProcessing.usf", "DownScaleTexture", SF_Pixel);
	IMPLEMENT_GLOBAL_SHADER(FUpscaleTexture, "/NeuralRendering/NeuralPostProcessing.usf", "UpscaleTexture", SF_Pixel);
	IMPLEMENT_GLOBAL_SHADER(FCopyBetweenTextureAndOverlappedTileBufferCS, "/NeuralRendering/NeuralPostProcessing.usf", "CopyBetweenTextureAndOverlappedTileBufferCS", SF_Compute);
}