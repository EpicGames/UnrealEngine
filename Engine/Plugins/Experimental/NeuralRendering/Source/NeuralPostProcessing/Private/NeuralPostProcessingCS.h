// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GlobalShader.h"
#include "ShaderParameterUtils.h"
#include "RenderGraphUtils.h"
#include "ScreenPass.h"

#define NEURAL_POST_PROCESSING_THREAD_GROUP_SIZE 32

namespace NeuralPostProcessng
{

	BEGIN_SHADER_PARAMETER_STRUCT(FNueralPostProcessInput, )
		SHADER_PARAMETER_STRUCT_INCLUDE(FScreenPassTextureViewportParameters, Viewport)
		SHADER_PARAMETER_RDG_TEXTURE(Texture2D, Texture)
		END_SHADER_PARAMETER_STRUCT()

	FNueralPostProcessInput GetNeuralPostProcessInput(FRDGTextureRef Texture, const FScreenPassTextureViewportParameters& ViewportParameters);

	class FNeuralPostProcessingBuildIndirectDispatchArgsCS : public FGlobalShader
	{
	public:
		DECLARE_GLOBAL_SHADER(FNeuralPostProcessingBuildIndirectDispatchArgsCS);
		SHADER_USE_PARAMETER_STRUCT(FNeuralPostProcessingBuildIndirectDispatchArgsCS, FGlobalShader)

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment);

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<uint>, RWIndirectDispatchArgsBuffer)
			SHADER_PARAMETER_RDG_BUFFER_SRV(Buffer<uint>, SourceType)
			SHADER_PARAMETER(FIntPoint, TargetDimension)
		END_SHADER_PARAMETER_STRUCT()

		static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters);
	};

	class FDownScaleTextureCS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FDownScaleTextureCS)
		SHADER_USE_PARAMETER_STRUCT(FDownScaleTextureCS, FGlobalShader)

	public:

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_STRUCT(FNueralPostProcessInput, Source)
			SHADER_PARAMETER_SAMPLER(SamplerState, SourceTextureSampler)
			SHADER_PARAMETER(int32,	TargetWidth)
			SHADER_PARAMETER(int32, TargetHeight)
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>, TargetTexture)
			RDG_BUFFER_ACCESS(IndirectDispatchArgsBuffer, ERHIAccess::IndirectArgs)
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);
	};

	class FDownScaleTexture : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FDownScaleTexture)
		SHADER_USE_PARAMETER_STRUCT(FDownScaleTexture, FGlobalShader)

	public:

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_STRUCT(FNueralPostProcessInput, Source)
			SHADER_PARAMETER_SAMPLER(SamplerState, SourceTextureSampler)
			SHADER_PARAMETER(int32, TargetWidth)
			SHADER_PARAMETER(int32, TargetHeight)
			RENDER_TARGET_BINDING_SLOTS()
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);
	};

	class FUpscaleTexture :public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FUpscaleTexture)
		SHADER_USE_PARAMETER_STRUCT(FUpscaleTexture, FGlobalShader)
	public:
		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER_RDG_TEXTURE(Texture2D<float4>, Source_Texture)
			SHADER_PARAMETER(int32, SourceWidth)
			SHADER_PARAMETER(int32, SourceHeight)
			SHADER_PARAMETER(FIntPoint,ViewportSize)
			RENDER_TARGET_BINDING_SLOTS()
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);
	};

	class FCopyBetweenTextureAndOverlappedTileBufferCS : public FGlobalShader
	{
		DECLARE_GLOBAL_SHADER(FCopyBetweenTextureAndOverlappedTileBufferCS)
		SHADER_USE_PARAMETER_STRUCT(FCopyBetweenTextureAndOverlappedTileBufferCS, FGlobalShader)

	public:

		BEGIN_SHADER_PARAMETER_STRUCT(FParameters, )
			SHADER_PARAMETER(int32, SourceWidth)
			SHADER_PARAMETER(int32, SourceHeight)
			SHADER_PARAMETER_RDG_TEXTURE_UAV(RWTexture2D<float4>,RWSourceTexture)
			SHADER_PARAMETER_SAMPLER(SamplerState, SourceTextureSampler)

			SHADER_PARAMETER(int32, TargetOverlappedTileWidth)
			SHADER_PARAMETER(int32, TargetOverlappedTileHeight)
			SHADER_PARAMETER(FIntPoint, ViewTileDimension)
			SHADER_PARAMETER(FVector2f, TileOverlap)
			SHADER_PARAMETER(int32, NumOfChannel)
			SHADER_PARAMETER(int32, bVisualizeOverlap)
			SHADER_PARAMETER(float, OverlapVisualizeIntensity)
			SHADER_PARAMETER_RDG_BUFFER_UAV(RWBuffer<float4>, TargetBuffer)

			RDG_BUFFER_ACCESS(IndirectDispatchArgsBuffer, ERHIAccess::IndirectArgs)
		END_SHADER_PARAMETER_STRUCT()

		static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& InParameters, FShaderCompilerEnvironment& OutEnvironment);

		enum class EDirection : uint32
		{
			ToOverlappedTiles,
			FromOverlappedTiles,
			MAX
		};

		enum class EOverlapResolveType : uint32
		{
			Ignore,
			Feathering,
			MAX
		};

		class FDimensionCopyDirection : SHADER_PERMUTATION_ENUM_CLASS("BUFFER_COPY_DIRECTION", EDirection);
		class FDimensionOverlapResolveType : SHADER_PERMUTATION_ENUM_CLASS("OVERLAP_RESOLVE_TYPE", EOverlapResolveType);
		using FPermutationDomain = TShaderPermutationDomain<FDimensionCopyDirection, FDimensionOverlapResolveType>;
	};
}