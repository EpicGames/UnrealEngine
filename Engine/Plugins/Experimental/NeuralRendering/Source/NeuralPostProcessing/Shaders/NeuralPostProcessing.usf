// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"
#include "/Engine/Private/Common.ush"
#include "/Engine/Private/ScreenPass.ush"

#define SOURCE_TYPE_TEXTURE		0
#define SOURCE_TYPE_BUFFER		1

#define BUFFER_COPY_TO_OVERLAPPING_TILE		0
#define BUFFER_COPY_FROM_OVERLAPPING_TILE	1

#ifndef BUFFER_COPY_DIRECTION
#define BUFFER_COPY_DIRECTION				BUFFER_COPY_TO_OVERLAPPING_TILE
#endif

#define OVERLAP_RESOLVE_IGNORE				0
#define OVERLAP_RESOLVE_FEATHERING			1

#ifndef OVERLAP_RESOLVE_TYPE
#define OVERLAP_RESOLVE_TYPE OVERLAP_RESOLVE_IGNORE
#endif

float2 ConvertGridPos2UV(uint2 GridPosition,float2 ExtentInverse)
{
	float2 GripPositionF = float2(GridPosition);
	return ExtentInverse * (GripPositionF + 0.5f);
}

Buffer<uint> SourceType;
RWBuffer<uint> RWIndirectDispatchArgsBuffer;
int2 TargetDimension;

int SourceWidth;
int SourceHeight;
int2 ViewportSize;
Texture2D<float4> Source_Texture;
SCREEN_PASS_TEXTURE_VIEWPORT(Source)
SamplerState SourceTextureSampler;

int TargetWidth;
int TargetHeight;
RWTexture2D<float4> TargetTexture;

int bVisualizeOverlap;
float OverlapVisualizeIntensity;

[numthreads(8, 1, 1)]
void BuildIndirectDispatchArgsCS(uint2 DispatchThreadId : SV_DispatchThreadID)
{
	if (DispatchThreadId.y == 0 && DispatchThreadId.x == 0)
	{
		if (SourceType[0] == SOURCE_TYPE_TEXTURE)
		{
			uint IndirectDispatchX = (TargetDimension.x + THREAD_GROUP_SIZE - 1) / THREAD_GROUP_SIZE;
			uint IndirectDispatchY = (TargetDimension.y + THREAD_GROUP_SIZE - 1) / THREAD_GROUP_SIZE;
			uint IndirectDispatchZ = 1;

			WriteDispatchIndirectArgs(RWIndirectDispatchArgsBuffer, 0, IndirectDispatchX, IndirectDispatchY, IndirectDispatchZ);
		}
		else
		{
			WriteDispatchIndirectArgs(RWIndirectDispatchArgsBuffer, 0, 0, 0, 1);
		}
	}
}

float3 BilinearInterpolate(float3 V00, float3 V10, float3 V01, float3 V11, float2 LerpFactors)
{
	float Alpha = LerpFactors.x;
	float Beta = LerpFactors.y;

	return (1.0 - Beta) * ((1.0 - Alpha) * V00 + Alpha * V10) + Beta * ((1.0 - Alpha) * V01 + Alpha * V11);
}

float4 BilinearInterpolate(float4 V00, float4 V10, float4 V01, float4 V11, float2 LerpFactors)
{
	float Alpha = LerpFactors.x;
	float Beta = LerpFactors.y;

	return (1.0 - Beta) * ((1.0 - Alpha) * V00 + Alpha * V10) + Beta * ((1.0 - Alpha) * V01 + Alpha * V11);
}

float4 UpscaleTexture(float4 Position : SV_POSITION) : SV_Target0
{
	const int2 BufferSize = int2(SourceWidth, SourceHeight);
	float X = Position.x * float(BufferSize.x - 1) / (float)ViewportSize.x;
	float Y = Position.y * float(BufferSize.y - 1) / (float)ViewportSize.y;

	int LoX = (int)clamp(floor(X), 0.0, float(BufferSize.x - 1));
	int HiX = (int)clamp(floor(X + 1.0), 0.0, float(BufferSize.x - 1));

	int LoY = (int)clamp(floor(Y), 0.0, float(BufferSize.y - 1));
	int HiY = (int)clamp(floor(Y + 1.0), 0.0, float(BufferSize.y - 1));

	// bilinear filtering
	float3 A = Source_Texture.Load(int3(LoX, LoY, 0)).xyz;
	float3 B = Source_Texture.Load(int3(HiX, LoY, 0)).xyz;
	float3 C = Source_Texture.Load(int3(LoX, HiY, 0)).xyz;
	float3 D = Source_Texture.Load(int3(HiX, HiY, 0)).xyz;

	float Alpha = clamp(X - (float)LoX, 0.0, 1.0);
	float Beta = clamp(Y - (float)LoY, 0.0, 1.0);

	//float3 Result = (1.0 - Beta) * ((1.0 - Alpha) * A + Alpha * B) + Beta * ((1.0 - Alpha) * C + Alpha * D);
	float3 Result = BilinearInterpolate(A, B, C, D, float2(Alpha, Beta));

	return float4(Result, 1.0);
}

float4 GetSourceGridAverageValue(float2 Position)
{
	const float2 BufferSize = float2(TargetWidth, TargetHeight);
	const float2 WidthFactors = Source_ViewportSize.xy / BufferSize;
	const int2 LoXY = Source_ViewportMin.xy + (int2)(WidthFactors * Position);
	const int2 HiXY = Source_ViewportMin.xy + (int2)ceil(WidthFactors * (Position + 1.0f));
	int LoX = LoXY.x;
	int LoY = LoXY.y;
	int HiX = HiXY.x;
	int HiY = HiXY.y;

	// averaging each grid
	float4 Result = float4(0.0, 0.0, 0.0, 0.0);
	float Div = 0.0;
	for (int x = LoX; x <= HiX; x++)
	{
		if (x < Source_Extent.x)
		{
			for (int y = LoY; y <= HiY; y++)
			{
				if (y < Source_Extent.y)
				{
					float2 BufferUV = ConvertGridPos2UV(uint2(x, y), Source_ExtentInverse);
					float2 ClampedBufferUV = clamp(BufferUV, Source_UVViewportBilinearMin, Source_UVViewportBilinearMax);
					Result += Source_Texture.SampleLevel(SourceTextureSampler, ClampedBufferUV, 0);
					Div += 1.0;
				}
			}
		}
	}

	Result /= Div;

	return Result;
}

float4 DownScaleTexture(float4 Position : SV_POSITION) : SV_Target0
{
	float4 Result = GetSourceGridAverageValue(Position.xy);
	return Result;
}

[numthreads(THREAD_GROUP_SIZE, THREAD_GROUP_SIZE, 1)]
void DownScaleTextureCS(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	if (DispatchThreadID.x >= TargetWidth || DispatchThreadID.y >= TargetHeight)
	{
		return;
	}

	float4 Result = GetSourceGridAverageValue(DispatchThreadID.xy);
	TargetTexture[DispatchThreadID.xy] = Result;
}

bool IsInRect(int2 Position, int2 MinPosition, int2 MaxPosition)
{
	return all(Position >= MinPosition) && all(Position < MaxPosition);
}

RWTexture2D<float4> RWSourceTexture;
int TargetOverlappedTileWidth;
int TargetOverlappedTileHeight;
float2 TileOverlap;
RWBuffer<float> TargetBuffer;
int2 ViewTileDimension;
int NumOfChannel;

#define TPVector2	float2

struct FTiledPosition
{
	int2 Index;		
	TPVector2 Position;

	// For a given Tiled Position, get the corresponding Position with respect to tile dimension and overlap
	// @TODO: float2 overlap
	TPVector2 ToPosition(int2 TileDimension, TPVector2 Overlaps)
	{
		const TPVector2 SourceTextureOffset = Index * TileDimension - Overlaps;
		return SourceTextureOffset + Position;
	}

	// Transform the current tiled position to the corresponding tiled position in the neighbor tile.
	FTiledPosition TransformBy(int2 Offset, int2 TileDimension, TPVector2 Overlaps)
	{
		FTiledPosition Neighbor = (FTiledPosition)0;
		Neighbor.Index = Index + Offset;
		TPVector2 AbsoluteNeighborPosition = Neighbor.ToPosition(TileDimension, Overlaps);
		TPVector2 AbsolutePosition = ToPosition(TileDimension, Overlaps);
		Neighbor.Position = AbsolutePosition - AbsoluteNeighborPosition;

		return Neighbor;
	}

	bool IsLeftEdge()
	{
		return Index.x == 0;
	}

	bool IsRightEdge()
	{
		return Index.x == (ViewTileDimension.x - 1);
	}

	bool IsTopEdge()
	{
		return Index.y == 0;
	}

	bool IsBottomEdge()
	{
		return Index.y == (ViewTileDimension.y - 1);
	}
};

FTiledPosition GetTiledPosition(int2 Position, int2 TileDimension)
{
	FTiledPosition TiledPosition = (FTiledPosition)0;
	TiledPosition.Index = Position / TileDimension;
	TiledPosition.Position = Position % TileDimension;

	return TiledPosition;
}

float4 SampleSourceTexture(TPVector2 SamplePosition,int2 TextureSize)
{
#if 0
	const float2 ExtentInverse = 1.0f/float2(SourceWidth, SourceHeight);
	float2  UV = ConvertGridPos2UV(SamplePosition, ExtentInverse);
	return  RWSourceTexture.SampleLevel(SourceTextureSampler, UV, 0);
#endif
	TPVector2 TextureSizeMax = TextureSize -1;
	TPVector2 MirrorPosition = abs(TextureSizeMax - abs(SamplePosition - TextureSizeMax));
	/*if (any(SamplePosition >= TextureSize) || any(SamplePosition < 0))
	{
		return 0.0f;
	}*/
	return RWSourceTexture[(uint2)MirrorPosition];
}

struct FBufferLocation
{
	int Offset;
	int ChannelSize;
	int Valid;
};

FBufferLocation GetBufferLocation(FTiledPosition TiledPosition,int2 TargetOverlappedTileDim)
{
	FBufferLocation BufferLocation = (FBufferLocation)0;
	BufferLocation.Valid = all(TiledPosition.Position >= 0) && 
		all(TiledPosition.Position < TargetOverlappedTileDim)&&
		all(TiledPosition.Index >= 0) &&
		all(TiledPosition.Index< ViewTileDimension);

	if (BufferLocation.Valid)
	{
		const int ChannelSize = TargetOverlappedTileDim.x * TargetOverlappedTileDim.y;
		const int BatchSize = ChannelSize * NumOfChannel;

		const int BatchIndex = TiledPosition.Index.x + TiledPosition.Index.y * ViewTileDimension.x;
		const int BatchOffset = BatchIndex * BatchSize;
		const int Offset = BatchOffset + (uint(TiledPosition.Position.x) + uint(TiledPosition.Position.y) * TargetOverlappedTileDim.x);


		BufferLocation.ChannelSize = ChannelSize;
		BufferLocation.Offset = Offset;
	}

	return BufferLocation;
}


void SetTargetBufferValue(FBufferLocation BufferLocation, float4 Value)
{
	for (int i = 0; i < NumOfChannel; ++i)
	{
		TargetBuffer[BufferLocation.Offset + i * BufferLocation.ChannelSize] = Value[i];
	}
}

// Use the default value if the Buffer location is not valid
float4 GetTargetBufferValue(FBufferLocation BufferLocation, float4 DefaultValue)
{
	float4 Value = DefaultValue;

	if (BufferLocation.Valid)
	{
		for (int i = 0; i < NumOfChannel; ++i)
		{
			Value[i] = TargetBuffer[BufferLocation.Offset + i * BufferLocation.ChannelSize];
		}
	}

	return Value;
}

float4 GetValueFromTransformedTile(FTiledPosition TiledPosition, int2 Offset, 
								   int2 TileDimension, TPVector2 InOverlap, float4 DefaultValue)
{
	FTiledPosition TransformedPosition = TiledPosition.TransformBy(Offset, TileDimension, InOverlap);
	FBufferLocation TransformedBufferLocation = GetBufferLocation(TransformedPosition, TileDimension + 2 * InOverlap);
	return GetTargetBufferValue(TransformedBufferLocation, DefaultValue);
}

float4 EmbedDebugValue(float4 Value, float4 DebugValue, bool ForceDebug)
{
	//@todo: compile time dynamic for debug view.
	return (bVisualizeOverlap||ForceDebug) ? (OverlapVisualizeIntensity*DebugValue) : Value;
}


[numthreads(THREAD_GROUP_SIZE, THREAD_GROUP_SIZE, 1)]
void CopyBetweenTextureAndOverlappedTileBufferCS(in const uint3 DispatchThreadID : SV_DispatchThreadID)
{
	const int2 TargetOverlappedTileDim = int2(TargetOverlappedTileWidth, TargetOverlappedTileHeight);
	const int2 TargetBufferDim = TargetOverlappedTileDim * ViewTileDimension;
	const int2 SourceTileDim = TargetOverlappedTileDim - 2 * TileOverlap;
	const int2 SourceBufferDim = min(SourceTileDim * ViewTileDimension, int2(SourceWidth, SourceHeight));

//#if BUFFER_COPY_DIRECTION == BUFFER_COPY_TO_OVERLAPPING_TILE
	if (any(DispatchThreadID.xy >= TargetBufferDim))
//#elif  BUFFER_COPY_DIRECTION == BUFFER_COPY_FROM_OVERLAPPING_TILE
//	if (any(DispatchThreadID.xy >= SourceBufferDim))
//#endif
	{
		return;
	}

	// Derive source texture position and buffer position
	const FTiledPosition TiledPosition = GetTiledPosition(DispatchThreadID.xy, TargetOverlappedTileDim);
	const TPVector2 TexturePosition = TiledPosition.ToPosition(SourceTileDim, TileOverlap);
	const FBufferLocation BufferLocation = GetBufferLocation(TiledPosition, TargetOverlappedTileDim);

#if BUFFER_COPY_DIRECTION == BUFFER_COPY_TO_OVERLAPPING_TILE

	float4 Value = SampleSourceTexture(TexturePosition, SourceBufferDim);
	SetTargetBufferValue(BufferLocation, Value);
	
#elif BUFFER_COPY_DIRECTION == BUFFER_COPY_FROM_OVERLAPPING_TILE
	// Read back from tiled buffer to buffer
	const uint2 WriteTexturePosition = (uint2)TexturePosition;

	// write pixel position is outside of the texture.
	if (any(WriteTexturePosition >= SourceBufferDim))
	{
		return;
	}

	const float4 Value0 = GetTargetBufferValue(BufferLocation, 0.0f);
	#if OVERLAP_RESOLVE_TYPE == OVERLAP_RESOLVE_IGNORE
		if (IsInRect(TiledPosition.Position, TileOverlap, TargetOverlappedTileDim - TileOverlap))
		{
			// Internal region with no overlapping with other tiles
			RWSourceTexture[WriteTexturePosition] = Value0;
		}
	#elif OVERLAP_RESOLVE_TYPE == OVERLAP_RESOLVE_FEATHERING
		if (IsInRect(TiledPosition.Position, 2*TileOverlap, TargetOverlappedTileDim - 2 * TileOverlap))
		{
			// Internal region with no overlapping with other tiles
			RWSourceTexture[WriteTexturePosition] = Value0;
		}
		else if (IsInRect(TiledPosition.Position, TileOverlap, TargetOverlappedTileDim - TileOverlap))
		{
			//Linear blending overlapping regions with other tiles. 
			if (IsInRect(TiledPosition.Position, TileOverlap + int2(0, TileOverlap.y), int2(2 * TileOverlap.x, TargetOverlappedTileDim.y - 2 * TileOverlap.y)))
			{
				//Section 1: lerping value in tile (x,y) and tile(x-1,y)
				float4 Value1 = GetValueFromTransformedTile(TiledPosition, int2(-1, 0), SourceTileDim, TileOverlap, Value0);

				float LerpFactor = 1 - TiledPosition.Position.x / (2.0f * TileOverlap.x);
				RWSourceTexture[WriteTexturePosition] = EmbedDebugValue(lerp(Value0, Value1, LerpFactor), 1.2f, false);
			}
			else if (IsInRect(TiledPosition.Position, 
				int2(TargetOverlappedTileDim.x - 2 * TileOverlap.x, 2 * TileOverlap.y),
				int2(TargetOverlappedTileDim.x - TileOverlap.x, TargetOverlappedTileDim.y - 2 * TileOverlap.y)))
			{
				//Section 2: lerping value in tile (x,y) and tile (x+1,y)
				float4 Value1 = GetValueFromTransformedTile(TiledPosition, int2(1, 0), SourceTileDim, TileOverlap, Value0);

				float LerpFactor = 1 - (TargetOverlappedTileDim.x - TiledPosition.Position.x) / (2.0f * TileOverlap.x);
				RWSourceTexture[WriteTexturePosition] = EmbedDebugValue(lerp(Value0, Value1, LerpFactor), float4(1.5, 0.0f, 0.0f, 1.0f), false);

			}
			else if (IsInRect(TiledPosition.Position,
				int2(2 * TileOverlap.x, TileOverlap.y),
				int2(TargetOverlappedTileDim.x - 2 * TileOverlap.x, 2 * TileOverlap.y)))
			{
				//Section 3: lerping value in tile (x,y) and tile (x,y-1)
				float4 Value1 = GetValueFromTransformedTile(TiledPosition, int2(0, -1), SourceTileDim, TileOverlap, Value0);

				float LerpFactor = 1 - TiledPosition.Position.y / (2.0f * TileOverlap.y);
				RWSourceTexture[WriteTexturePosition] = EmbedDebugValue(lerp(Value0, Value1, LerpFactor), float4(0.0, 0.8f, 0.0f, 1.0f), false);
			}
			else if (IsInRect(TiledPosition.Position,
				int2(2 * TileOverlap.x, TargetOverlappedTileDim.y - 2 * TileOverlap.y),
				int2(TargetOverlappedTileDim.x - 2 * TileOverlap.x, TargetOverlappedTileDim.y - TileOverlap.y)))
			{
				//Section 4: lerping value in tile (x,y) and tile (x,y+1)
				float4 Value1 = GetValueFromTransformedTile(TiledPosition, int2(0, 1), SourceTileDim, TileOverlap, Value0);
				
				float LerpFactor = 1 - (TargetOverlappedTileDim.y - TiledPosition.Position.y) / (2.0f * TileOverlap.y);
				RWSourceTexture[WriteTexturePosition] = EmbedDebugValue(lerp(Value0, Value1, LerpFactor), float4(0.0, 0.2f, 0.9f, 1.0f), false);
			}
			else if (IsInRect(TiledPosition.Position,
				TileOverlap, 2 * TileOverlap))
			{
				//Section 5: lerping value in tile (x,y) and tile (x-1,y-1),(x,y-1),(x-1,y)
				// V00(-1, -1), V10(0, -1)
				// V01(-1,  0), V11(0,  0)
				// Assume horizontal edge
				float4 V11 = Value0;
				float4 V01 = GetValueFromTransformedTile(TiledPosition, int2(-1,  0), SourceTileDim, TileOverlap, Value0);
				float4 V00 = GetValueFromTransformedTile(TiledPosition, int2(-1, -1), SourceTileDim, TileOverlap,    V01);
				float4 V10 = GetValueFromTransformedTile(TiledPosition, int2( 0, -1), SourceTileDim, TileOverlap, Value0);
				// Check left edge
				if (TiledPosition.IsLeftEdge())
				{
					V00 = V10;
				}

				float2 LerpFactors = TiledPosition.Position/(2.0f * TileOverlap);
				// bilinear interpolation
				float4 FilterdValue = BilinearInterpolate(V00, V10, V01, V11, LerpFactors);
				RWSourceTexture[WriteTexturePosition] = EmbedDebugValue(FilterdValue, float4(1.0, 1.2f, 0.2f, 1.0f), false);
			}
			else if (IsInRect(TiledPosition.Position,
				int2(TargetOverlappedTileDim.x-3*TileOverlap.x,0)+ TileOverlap, 
				int2(TargetOverlappedTileDim.x - 3 * TileOverlap.x, 0) + 2 * TileOverlap))
			{
				//Section 6: lerping value in tile (x,y) and tile (x+1,y-1),(x+1,y),(x,y-1)
				// V00(0, -1), V10(1, -1)
				// V01(0,  0), V11(1,  0)
				// Assume horizontal edge
				float4 V00 = GetValueFromTransformedTile(TiledPosition, int2( 0, -1), SourceTileDim, TileOverlap, Value0);
				float4 V11 = GetValueFromTransformedTile(TiledPosition, int2( 1,  0), SourceTileDim, TileOverlap, Value0);
				float4 V01 = Value0;
				float4 V10 = GetValueFromTransformedTile(TiledPosition, int2(1, -1), SourceTileDim, TileOverlap,     V11);
				// Check right edge
				if (TiledPosition.IsRightEdge())
				{
					V10 = V00;
				}

				float2 LerpFactors = (TiledPosition.Position - int2(SourceTileDim.x,0)) / (2.0f * TileOverlap);

				// bilinear interpolation
				float4 FilterdValue = BilinearInterpolate(V00, V10, V01, V11, LerpFactors);
				RWSourceTexture[WriteTexturePosition] = EmbedDebugValue(FilterdValue, float4(2.0, 0.2f, 0.9f, 1.0f), false);
			}
			else if (IsInRect(TiledPosition.Position,
				int2(TileOverlap.x, TargetOverlappedTileDim.y - 2 * TileOverlap.y),
				int2(2 * TileOverlap.x, TargetOverlappedTileDim.y - TileOverlap.y)))
			{
				//Section 7: lerping value in tile (x,y) and tile (x-1,y),(x-1,y+1),(x,y+1)
				// V00(-1,  0), V10(0, 0)
				// V01(-1,  1), V11(0, 1)
				// Assume horizontal edge
				float4 V00 = GetValueFromTransformedTile(TiledPosition, int2(-1, 0), SourceTileDim, TileOverlap, Value0);
				float4 V10 = Value0;
				float4 V01 = GetValueFromTransformedTile(TiledPosition, int2(-1, 1), SourceTileDim, TileOverlap,    V00);
				float4 V11 = GetValueFromTransformedTile(TiledPosition, int2( 0, 1), SourceTileDim, TileOverlap, Value0);
				float2 LerpFactors = (TiledPosition.Position - int2(0, SourceTileDim.y)) / (2.0f * TileOverlap);
				// Check left edge
				if (TiledPosition.IsLeftEdge())
				{
					V01 = V11;
				}

				// bilinear interpolation
				float4 FilterdValue = BilinearInterpolate(V00, V10, V01, V11, LerpFactors);
				RWSourceTexture[WriteTexturePosition] = EmbedDebugValue(FilterdValue, float4(0.0, 2.2f, 2.9f, 1.0f), false);
			}
			else if (IsInRect(TiledPosition.Position,
				int2(TargetOverlappedTileDim.x -2 * TileOverlap.x, TargetOverlappedTileDim.y - 2 * TileOverlap.y),
				int2(TargetOverlappedTileDim.x - TileOverlap.x, TargetOverlappedTileDim.y - TileOverlap.y)))
			{
				//Section 8: lerping value in tile (x,y) and tile (x,y+1),(x+1,y),(x+1,y+1)
				// V00( 0,  0), V10( 1, 0)
				// V01( 0,  1), V11( 1, 1)
				// Assume horizontal edge
				float4 V00 = Value0;
				float4 V10 = GetValueFromTransformedTile(TiledPosition, int2( 1, 0), SourceTileDim, TileOverlap, Value0);
				float4 V01 = GetValueFromTransformedTile(TiledPosition, int2( 0, 1), SourceTileDim, TileOverlap,    V00);
				float4 V11 = GetValueFromTransformedTile(TiledPosition, int2( 1, 1), SourceTileDim, TileOverlap,    V10);
				float2 LerpFactors = (TiledPosition.Position - SourceTileDim) / (2.0f * TileOverlap);
				// Check right edge
				if (TiledPosition.IsRightEdge())
				{
					V11 = V01;
				}

				// bilinear interpolation
				float4 FilterdValue = BilinearInterpolate(V00, V10, V01, V11, LerpFactors);
				RWSourceTexture[WriteTexturePosition] = EmbedDebugValue(FilterdValue, float4(2.0, 2.2f, 0.1f, 1.0f), false);
			}
		}
		else
		{
			// border regions that should be ignored
		}
	#endif// End of OVERLAP_RESOLVE_TYPE
#endif

}
